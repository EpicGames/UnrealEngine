// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioMixer.h"

#include "Async/Async.h"
#include "AudioDefines.h"
#include "AudioMixerTrace.h"
#include "DSP/AudioChannelUtils.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeTryLock.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Trace/Trace.h"


// Defines the "Audio" category in the CSV profiler.
// This should only be defined here. Modules who wish to use this category should contain the line
// 		CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);
//
CSV_DEFINE_CATEGORY_MODULE(AUDIOMIXERCORE_API, Audio, false);

#if UE_AUDIO_PROFILERTRACE_ENABLED
UE_TRACE_CHANNEL_DEFINE(AudioChannel, "AudioInsights logic data (such as asset load/unload/start/stop, etc.)");
UE_TRACE_CHANNEL_DEFINE(AudioMixerChannel, "AudioInsights parameter data (such as amplitude, pitch, etc.)");
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

// Audio Insights Event Log in-built event types
namespace Audio::Trace::EventLog::ID
{
	// Active Sound : Playing Audio
	const FString SoundStart = TEXT("Playing");
	const FString SoundStop = TEXT("Stopped");

	// Virtualization
	const FString SoundVirtualized = TEXT("Virtualized");
	const FString SoundRealized = TEXT("Realized");

	// Triggers
	const FString PlayRequestSoundHandle = TEXT("Play Request : Sound Handle");
	const FString StopRequestedSoundHandle = TEXT("Stop Request : Sound Handle");

	const FString PlayRequestAudioComponent = TEXT("Play Request : Audio Component");
	const FString StopRequestAudioComponent = TEXT("Stop Request : Audio Component");

	const FString PlayRequestOneShot = TEXT("Play Request : One shot");
	const FString PlayRequestSoundAtLocation = TEXT("Play Request : Sound at location");
	const FString PlayRequestSound2D = TEXT("Play Request : Play Sound 2D");
	const FString PlayRequestSlateSound = TEXT("Play Request : Slate Sound");

	const FString StopRequestActiveSound = TEXT("Stop Request : Active Sound");
	const FString StopRequestSoundsUsingResource = TEXT("Stop Request : Sounds using resource");
	const FString StopRequestConcurrency = TEXT("Stop Request : Concurrency");

	const FString PauseSoundRequested = TEXT("Paused");
	const FString ResumeSoundRequested = TEXT("Resumed");

	const FString StopAllRequested = TEXT("Stop All");
	const FString FlushAudioDeviceRequested = TEXT("Flush Audio Device");

	const FString PlayFailedNotPlayable = TEXT("Play Failed : Not playable");
	const FString PlayFailedOutOfRange = TEXT("Play Failed : Out of range");
	const FString PlayFailedDebugFiltered = TEXT("Play Failed : Debug filtered");
	const FString PlayFailedConcurrency = TEXT("Play Failed : Concurrency");

	// PIE Session
	const FString PIEStarted = TEXT("PIE Started");
	const FString PIEStopped = TEXT("PIE Stopped");
}

// Command to enable logging to display accurate audio render times
static int32 LogRenderTimesCVar = 0;
FAutoConsoleVariableRef CVarLogRenderTimes(
	TEXT("au.LogRenderTimes"),
	LogRenderTimesCVar,
	TEXT("Logs Audio Render Times.\n")
	TEXT("0: Not Log, 1: Log"),
	ECVF_Default);

static float MinTimeBetweenUnderrunWarningsMs = 1000.f*10.f;
FAutoConsoleVariableRef CVarMinTimeBetweenUnderrunWarningsMs(
	TEXT("au.MinLogTimeBetweenUnderrunWarnings"),
	MinTimeBetweenUnderrunWarningsMs,
	TEXT("Min time between underrun warnings (globally) in MS\n")
	TEXT("Set the time between each subsequent underrun log warning globaly (defaults to 10secs)"),
	ECVF_Default);

// Command for setting the audio render thread priority.
static int32 SetRenderThreadPriorityCVar = (int32)TPri_Highest;
FAutoConsoleVariableRef CVarSetRenderThreadPriority(
	TEXT("au.RenderThreadPriority"),
	SetRenderThreadPriorityCVar,
	TEXT("Sets audio render thread priority. Defaults to 3.\n")
	TEXT("0: Normal, 1: Above Normal, 2: Below Normal, 3: Highest, 4: Lowest, 5: Slightly Below Normal, 6: Time Critical"),
	ECVF_Default);

static int32 SetRenderThreadAffinityCVar = 0;
FAutoConsoleVariableRef CVarRenderThreadAffinity(
	TEXT("au.RenderThreadAffinity"),
	SetRenderThreadAffinityCVar,
	TEXT("Override audio render thread affinity.\n")
	TEXT("0: Disabled (Default), otherwise overriden thread affinity."),
	ECVF_Default);

static int32 bUseThreadedDeviceSwapCVar = 1;
FAutoConsoleVariableRef CVarUseThreadedDeviceSwap(
	TEXT("au.UseThreadedDeviceSwap"),
	bUseThreadedDeviceSwapCVar,
	TEXT("Lets Device Swap go wide.")
	TEXT("0 off, 1 on"),
	ECVF_Default);

static int32 bUseAudioDeviceInfoCacheCVar = 1;
FAutoConsoleVariableRef CVarUseAudioDeviceInfoCache(
	TEXT("au.UseCachedDeviceInfoCache"),
	bUseAudioDeviceInfoCacheCVar,
	TEXT("Uses a Cache of the DeviceCache instead of asking the OS")
	TEXT("0 off, 1 on"),
	ECVF_Default);
	
static int32 bRecycleThreadsCVar = 1;
FAutoConsoleVariableRef CVarRecycleThreads(
	TEXT("au.RecycleThreads"),
	bRecycleThreadsCVar,
	TEXT("Keeps threads to reuse instead of create/destroying them")
	TEXT("0 off, 1 on"),
	ECVF_Default);

static int32 OverrunTimeoutCVar = 5000;
FAutoConsoleVariableRef CVarOverrunTimeout(
	TEXT("au.OverrunTimeoutMSec"),
	OverrunTimeoutCVar,
	TEXT("Amount of time to wait for the render thread to time out before swapping to the null device. \n"),
	ECVF_Default);

static int32 UnderrunTimeoutCVar = 5;
FAutoConsoleVariableRef CVarUnderrunTimeout(
	TEXT("au.UnderrunTimeoutMSec"),
	UnderrunTimeoutCVar,
	TEXT("Amount of time to wait for the render thread to generate the next buffer before submitting an underrun buffer. \n"),
	ECVF_Default);

static int32 FadeoutTimeoutCVar = 2000;
FAutoConsoleVariableRef CVarFadeoutTimeout(
	TEXT("au.FadeOutTimeoutMSec"),
	FadeoutTimeoutCVar,
	TEXT("Amount of time to wait for the FadeOut Event to fire. \n"),
	ECVF_Default);

static float LinearGainScalarForFinalOututCVar = 1.0f;
FAutoConsoleVariableRef LinearGainScalarForFinalOutut(
	TEXT("au.LinearGainScalarForFinalOutut"),
	LinearGainScalarForFinalOututCVar,
	TEXT("Linear gain scalar applied to the final float buffer to allow for hotfixable mitigation of clipping \n")
	TEXT("Default is 1.0f \n"),
	ECVF_Default);

static int32 ExtraAudioMixerDeviceLoggingCVar = 0;
FAutoConsoleVariableRef ExtraAudioMixerDeviceLogging(
	TEXT("au.ExtraAudioMixerDeviceLogging"),
	ExtraAudioMixerDeviceLoggingCVar,
	TEXT("Enables extra logging for audio mixer device running \n")
	TEXT("0: no logging, 1: logging every 500 callbacks \n"),
	ECVF_Default);

static int32 AudioMixerDebugForceDroppedHardwareCallbackCVar = 0;
FAutoConsoleVariableRef CVarAudioMixerDebugForceDroppedHardwareCallback(
	TEXT("au.debug.ForceDroppedHardwareCallback"),
	AudioMixerDebugForceDroppedHardwareCallbackCVar,
	TEXT("Will drop the next N hardware callbacks\n")
	TEXT("N == 0 off, N < 0 (will drop all callbacks), N > 0 will drop (will drop N callbacks)"),
	ECVF_Cheat);

// Stat definitions for profiling audio mixer 
DEFINE_STAT(STAT_AudioMixerRenderAudio);

namespace Audio
{
	int32 sRenderInstanceIds = 0;

	FThreadSafeCounter AudioMixerTaskCounter;

	FAudioRenderTimeAnalysis::FAudioRenderTimeAnalysis()
		: AvgRenderTime(0.0)
		, MaxRenderTime(0.0)
		, TotalRenderTime(0.0)
		, StartTime(0.0)
		, RenderTimeCount(0)
		, RenderInstanceId(sRenderInstanceIds++)
	{}

	void FAudioRenderTimeAnalysis::Start()
	{
		StartTime = FPlatformTime::Cycles();
	}

	void FAudioRenderTimeAnalysis::End()
	{
		uint32 DeltaCycles = FPlatformTime::Cycles() - StartTime;
		double DeltaTime = DeltaCycles * FPlatformTime::GetSecondsPerCycle();

		TotalRenderTime += DeltaTime;
		RenderTimeSinceLastLog += DeltaTime;
		++RenderTimeCount;
		AvgRenderTime = TotalRenderTime / RenderTimeCount;
		
		if (DeltaTime > MaxRenderTime)
		{
			MaxRenderTime = DeltaTime;
		}
		
		if (DeltaTime > MaxSinceTick)
		{
			MaxSinceTick = DeltaTime;
		}

		if (LogRenderTimesCVar == 1)
		{
			if (RenderTimeCount % 32 == 0)
			{
				RenderTimeSinceLastLog /= 32.0f;
				UE_LOGF(LogAudioMixerDebug, Display, "Render Time [id:%d] - Max: %.2f ms, MaxDelta: %.2f ms, Delta Avg: %.2f ms, Global Avg: %.2f ms", 
					RenderInstanceId, 
					(float)MaxRenderTime * 1000.0f, 
					(float)MaxSinceTick * 1000.0f,
					RenderTimeSinceLastLog * 1000.0f, 
					(float)AvgRenderTime * 1000.0f);

				RenderTimeSinceLastLog = 0.0f;
				MaxSinceTick = 0.0f;
			}
		}
	}


	void FOutputBuffer::Init(IAudioMixer* InAudioMixer, const int32 InNumSamples, const int32 InNumBuffers, const EAudioMixerStreamDataFormat::Type InDataFormat)
	{
		SCOPED_NAMED_EVENT(FOutputBuffer_Init, FColor::Blue);

		RenderBuffer.Reset();
		RenderBuffer.AddUninitialized(InNumSamples);

		DataFormat = InDataFormat;

		check(InAudioMixer != nullptr);
		AudioMixer = InAudioMixer;

		CircularBuffer.SetCapacity(InNumSamples * InNumBuffers * GetSizeForDataFormat(DataFormat));

		PopBuffer.Reset();
		PopBuffer.AddUninitialized(InNumSamples * GetSizeForDataFormat(DataFormat));

		if (DataFormat != EAudioMixerStreamDataFormat::Float)
		{
			FormattedBuffer.SetNumZeroed(InNumSamples * GetSizeForDataFormat(DataFormat));
		}
	}

	bool FOutputBuffer::MixNextBuffer()
 	{
		// If the circular queue is already full, exit.
		const int32 SpaceLeftInBufferInSamples = CircularBuffer.Remainder() / GetSizeForDataFormat(DataFormat);
		if (SpaceLeftInBufferInSamples < RenderBuffer.Num())
		{
			return false;
		}

		CSV_SCOPED_TIMING_STAT(Audio, RenderAudio);
		SCOPE_CYCLE_COUNTER(STAT_AudioMixerRenderAudio);

		// Zero the buffer
		FPlatformMemory::Memzero(RenderBuffer.GetData(), RenderBuffer.Num() * sizeof(float));
		if (AudioMixer != nullptr)
		{
			AudioMixer->OnProcessAudioStream(RenderBuffer);
		}

		switch (DataFormat)
		{
		case EAudioMixerStreamDataFormat::Float:
		{
			if (!FMath::IsNearlyEqual(LinearGainScalarForFinalOututCVar, 1.0f))
			{
				ArrayMultiplyByConstantInPlace(RenderBuffer, LinearGainScalarForFinalOututCVar);
			}
			ArrayRangeClamp(RenderBuffer, -1.0f, 1.0f);

			// No conversion is needed, so we push the RenderBuffer directly to the circular queue.
			CircularBuffer.Push(reinterpret_cast<const uint8*>(RenderBuffer.GetData()), RenderBuffer.Num() * sizeof(float));
		}
		break;

		case EAudioMixerStreamDataFormat::Int16:
		{
			int16* BufferInt16 = (int16*)FormattedBuffer.GetData();
			const int32 NumSamples = RenderBuffer.Num();
			check(FormattedBuffer.Num() / GetSizeForDataFormat(DataFormat) == RenderBuffer.Num());			

			const float ConversionScalar = LinearGainScalarForFinalOututCVar * 32767.0f;
			ArrayMultiplyByConstantInPlace(RenderBuffer, ConversionScalar);
			ArrayRangeClamp(RenderBuffer, -32767.0f, 32767.0f);

			for (int32 i = 0; i < NumSamples; ++i)
			{
				BufferInt16[i] = (int16)RenderBuffer[i];
			}

			CircularBuffer.Push(reinterpret_cast<const uint8*>(FormattedBuffer.GetData()), FormattedBuffer.Num());
		}
		break;

		default:
			// Not implemented/supported
			check(false);
			break;
		}

		static const int32 HeartBeatRate = 500;
		if ((ExtraAudioMixerDeviceLoggingCVar > 0) && (++CallCounterMixNextBuffer > HeartBeatRate))
		{
			UE_LOGF(LogAudioMixer, Display, "FOutputBuffer::MixNextBuffer() called %i times", HeartBeatRate);
			CallCounterMixNextBuffer = 0;
		}

		return true;
 	}
 
	TArrayView<const uint8> FOutputBuffer::PopBufferData(int32& OutNumBytesPopped) const
	{
		FMemory::Memzero(reinterpret_cast<uint8*>(PopBuffer.GetData()), PopBuffer.Num());
		OutNumBytesPopped = CircularBuffer.Pop(PopBuffer.GetData(), PopBuffer.Num());

		return TArrayView<const uint8>(PopBuffer);
	}

	int32 FOutputBuffer::GetNumSamples() const
	{
		return RenderBuffer.Num();
	}

	size_t FOutputBuffer::GetSizeForDataFormat(EAudioMixerStreamDataFormat::Type InDataFormat)
	{
		switch (InDataFormat)
		{
		case EAudioMixerStreamDataFormat::Float:
			return sizeof(float);

		case EAudioMixerStreamDataFormat::Int16:
			return sizeof(int16);

		default:
			checkNoEntry();
			return 0;
		}
	}

	/**
	 * IAudioMixerPlatformInterface
	 */

	// Static linkage.
	FThreadSafeCounter IAudioMixerPlatformInterface::NextInstanceID;

	IAudioMixerPlatformInterface::IAudioMixerPlatformInterface()
		: bWarnedBufferUnderrun(false)
		, AudioRenderEvent(nullptr)
		, AudioFadeEvent(nullptr)
		, NumOutputBuffers(0)
		, FadeVolume(0.0f)
		, LastError(TEXT("None"))
		, bPerformingFade(true)
		, bFadedOut(false)
		, bIsDeviceInitialized(false)
		, bIsUsingNullDevice(false)
		, bIsGeneratingAudio(false)
		, InstanceID(NextInstanceID.Increment())
		, NullDeviceCallback(nullptr)
	{
		FadeParam.SetValue(0.0f);
	}

	IAudioMixerPlatformInterface::~IAudioMixerPlatformInterface()
	{
		check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Closed);
	}

	void IAudioMixerPlatformInterface::FadeIn()
	{
		if (IsNonRealtime())
		{
			FadeParam.SetValue(1.0f);
		}

		bPerformingFade = true;
		bFadedOut = false;
		FadeVolume = 1.0f;
	}

	void IAudioMixerPlatformInterface::FadeOut()
	{
		// Non Realtime isn't ticked when fade out is called, and the user can't hear
		// the output anyways so there's no need to make it pleasant for their ears.
		if (!FPlatformProcess::SupportsMultithreading() || IsNonRealtime())
		{
			bFadedOut = true;
			FadeVolume = 0.f;
			return;
		}

		if (bFadedOut || FadeVolume == 0.0f)
		{
			return;
		}

		bPerformingFade = true;
		if (AudioFadeEvent != nullptr)
		{						
			if (!AudioFadeEvent->Wait(FadeoutTimeoutCVar))
			{
				UE_LOGF(LogAudioMixer, Warning, "FadeOutEvent timed out");
			}
		}

		FadeVolume = 0.0f;
	}

	void IAudioMixerPlatformInterface::PostInitializeHardware()
	{
		bIsDeviceInitialized = true;
	}

	int32 IAudioMixerPlatformInterface::GetIndexForDevice(const FString& InDeviceName)
	{
		uint32 TotalNumDevices = 0;

		if (!GetNumOutputDevices(TotalNumDevices))
		{
			return INDEX_NONE;
		}

		// Iterate through every device and see if
		for (uint32 DeviceIndex = 0; DeviceIndex < TotalNumDevices; DeviceIndex++)
		{
			FAudioPlatformDeviceInfo DeviceInfo;
			if (GetOutputDeviceInfo(DeviceIndex, DeviceInfo))
			{
				// check if the device name matches the input device name:
				if (DeviceInfo.Name.Contains(InDeviceName))
				{
					return DeviceIndex;
				}
			}
		}

		// If we've made it here, we weren't able to find a matching device.
		return INDEX_NONE;
	}

	template<typename BufferType>
	void IAudioMixerPlatformInterface::ApplyAttenuationInternal(TArrayView<BufferType>& InOutBuffer)
	{
		static const int32 HeartBeatRate = 500;
		const bool bLog = (ExtraAudioMixerDeviceLoggingCVar > 0) && (++CallCounterApplyAttenuationInternal > HeartBeatRate);
		if (bLog)
		{
			UE_LOGF(LogAudioMixer, Display, "IAudioMixerPlatformInterface::ApplyAttenuationInternal() called %i times", HeartBeatRate);
			CallCounterApplyAttenuationInternal = 0;
		}

		// Perform fade in and fade out global attenuation to avoid clicks/pops on startup/shutdown
		if (bPerformingFade)
		{
			FadeParam.SetValue(FadeVolume, InOutBuffer.Num());

			for (int32 i = 0; i < InOutBuffer.Num(); ++i)
			{
				InOutBuffer[i] = (BufferType)(InOutBuffer[i] * FadeParam.Update());
			}

			bFadedOut = (FadeVolume == 0.0f);
			bPerformingFade = false;
			if (AudioFadeEvent)
			{
				AudioFadeEvent->Trigger();
			}
			if (bLog)
			{
				UE_LOGF(LogAudioMixer, Display, "IAudioMixerPlatformInterface::ApplyAttenuationInternal() Faded from %f to %f", FadeVolume, FadeParam.GetValue());
			}
		}
		else if (bFadedOut)
		{
			// If we're faded out, then just zero the data.
			FPlatformMemory::Memzero((void*)InOutBuffer.GetData(), sizeof(BufferType)* InOutBuffer.Num());

			if (bLog)
			{
				UE_LOGF(LogAudioMixer, Display, "IAudioMixerPlatformInterface::ApplyAttenuationInternal() Zero'd out buffer");
			}
		}

		FadeParam.Reset();
	}

	void IAudioMixerPlatformInterface::StartRunningNullDevice()
	{
		UE_LOGF(LogAudioMixer, Display, "StartRunningNullDevice() called, InstanceID=%d", InstanceID);
		SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_StartRunningNullDevice, FColor::Blue);
		
		auto ThrowAwayBuffer = [this]() { this->ReadNextBuffer(); };
		float SafeSampleRate = OpenStreamParams.SampleRate > 0.f ? OpenStreamParams.SampleRate : 48000.f;
		float BufferDuration = ((float)OpenStreamParams.NumFrames) / SafeSampleRate;

		if (AudioRenderEvent)
		{
			AudioRenderEvent->Trigger();
		}

		if (!NullDeviceCallback.IsValid())
		{
			// Create the thread and tell it not to pause.
			CreateNullDeviceThread(ThrowAwayBuffer, BufferDuration, false);
			check(NullDeviceCallback.IsValid());
		}
		else
		{
			// Reuse existing thread if we have one.
			NullDeviceCallback->Resume(ThrowAwayBuffer, BufferDuration);
		}

		bIsUsingNullDevice = true;
	}

	void IAudioMixerPlatformInterface::StopRunningNullDevice()
	{		
		UE_LOGF(LogAudioMixer, Display, "StopRunningNullDevice() called, InstanceID=%d", InstanceID);
		SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_StopRunningNullDevice, FColor::Blue);

		if (NullDeviceCallback.IsValid())
		{
			if(IAudioMixer::ShouldRecycleThreads())
			{
				NullDeviceCallback->Pause();
			}
			else
			{
				NullDeviceCallback.Reset();
			}
			bIsUsingNullDevice = false;
		}
	}

	void IAudioMixerPlatformInterface::CreateNullDeviceThread(const TFunction<void()> InCallback, float InBufferDuration, bool bShouldPauseOnStart)
	{
		NullDeviceCallback.Reset(new FMixerNullCallback(InBufferDuration, InCallback, TPri_TimeCritical, bShouldPauseOnStart));
	}

	void IAudioMixerPlatformInterface::ApplyPrimaryAttenuation(TArrayView<const uint8>& OutPoppedAudio)
	{
		EAudioMixerStreamDataFormat::Type Format = OutputBuffer.GetFormat();

		if (Format == EAudioMixerStreamDataFormat::Float)
		{
			TArrayView<float> OutFloatBuffer = TArrayView<float>(const_cast<float*>(reinterpret_cast<const float*>(OutPoppedAudio.GetData())), OutPoppedAudio.Num() / sizeof(float));
			ApplyAttenuationInternal(OutFloatBuffer);
		}
		else if (Format == EAudioMixerStreamDataFormat::Int16)
		{
			TArrayView<int16> OutIntBuffer = TArrayView<int16>(const_cast<int16*>(reinterpret_cast<const int16*>(OutPoppedAudio.GetData())), OutPoppedAudio.Num() / sizeof(int16));
			ApplyAttenuationInternal(OutIntBuffer);
		}
		else
		{
			checkNoEntry();
		}
	}

	void IAudioMixerPlatformInterface::ReadNextBuffer()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

#if !UE_BUILD_SHIPPING
		// Debugging CVAR to drop hardware calls.
		if (AudioMixerDebugForceDroppedHardwareCallbackCVar != 0)
		{
			// Drop N calls...
			if (AudioMixerDebugForceDroppedHardwareCallbackCVar > 0)
			{
				AudioMixerDebugForceDroppedHardwareCallbackCVar--;
			}
			return;
		}
#endif //!UE_BUILD_SHIPPING

		// If we are currently swapping devices and OnBufferEnd is being triggered in an XAudio2Thread,
		// early exit.
		if (!DeviceSwapCriticalSection.TryLock())
		{
			return;
		}

		// Don't read any more audio if we're not running
		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Running)
		{
			DeviceSwapCriticalSection.Unlock();
			return;
		}
		
	
		int32 NumSamplesPopped = 0;
		TArrayView<const uint8> PoppedAudio = OutputBuffer.PopBufferData(NumSamplesPopped);

		bool bDidOutputUnderrun = NumSamplesPopped != PoppedAudio.Num();
		
		if (bDidOutputUnderrun)
		{
			UnderrunCount++;
			CurrentUnderrunCount++;
			
			if (!bWarnedBufferUnderrun)
			{
				float ElapsedTimeInMs = static_cast<float>(FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - TimeLastWarningCycles));
				if( ElapsedTimeInMs > MinTimeBetweenUnderrunWarningsMs )
				{
					// Underrun/Starvation:
					// Things to try: Increase # output buffers, ensure audio-render thread has time to run (affinity and priority), debug your mix and reduce # sounds playing.

					UE_LOGF(LogAudioMixer, Display, "Audio Buffer Underrun (starvation) detected. InstanceID=%d", InstanceID);
					bWarnedBufferUnderrun = true;
					TimeLastWarningCycles = FPlatformTime::Cycles64();
				}
			}
		}
		else
		{
			// As soon as a valid buffer goes through, allow more warning
			if (bWarnedBufferUnderrun)
			{
				UE_LOGF(LogAudioMixerDebug, Log, "Audio had %d underruns [Total: %d], InstanceID=%d", CurrentUnderrunCount, UnderrunCount, InstanceID);
			}

			CurrentUnderrunCount = 0;
			bWarnedBufferUnderrun = false;
		}

		ApplyPrimaryAttenuation(PoppedAudio);
		SubmitBuffer(PoppedAudio.GetData());

		DeviceSwapCriticalSection.Unlock();

		// Kick off rendering of the next set of buffers
		if (AudioRenderEvent)
		{
			AudioRenderEvent->Trigger();
		}
	}

	void IAudioMixerPlatformInterface::BeginGeneratingAudio()
	{
		SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_BeginGeneratingAudio, FColor::Blue);
		
		checkf(!bIsGeneratingAudio, TEXT("BeginGeneratingAudio() is being run with StreamState = %i and bIsGeneratingAudio = %i"), AudioStreamInfo.StreamState.load(), !!bIsGeneratingAudio);

		bIsGeneratingAudio = true;

		// Setup the output buffers
		const int32 NumOutputFrames = OpenStreamParams.NumFrames;
		const int32 NumOutputChannels = AudioStreamInfo.DeviceInfo.NumChannels;
		const int32 NumOutputSamples = NumOutputFrames * NumOutputChannels;

		// Set the number of buffers to be one more than the number to queue.
		NumOutputBuffers = FMath::Max(OpenStreamParams.NumBuffers, 2);
		UE_LOGF(LogAudioMixer, Display, "Output buffers initialized: Frames=%i, Channels=%i, Samples=%i, InstanceID=%d", NumOutputFrames, NumOutputChannels, NumOutputSamples, InstanceID);


		OutputBuffer.Init(AudioStreamInfo.AudioMixer, NumOutputSamples, NumOutputBuffers, AudioStreamInfo.DeviceInfo.Format);

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Running;

		check(AudioRenderEvent == nullptr);
		AudioRenderEvent = FPlatformProcess::GetSynchEventFromPool();
		check(AudioRenderEvent != nullptr);

		check(AudioFadeEvent == nullptr);
		AudioFadeEvent = FPlatformProcess::GetSynchEventFromPool();
		check(AudioFadeEvent != nullptr);

		check(!AudioRenderThread.IsValid());
		uint64 RenderThreadAffinityCVar = SetRenderThreadAffinityCVar > 0 ? uint64(SetRenderThreadAffinityCVar) : FPlatformAffinity::GetAudioRenderThreadMask();
		AudioRenderThread.Reset(FRunnableThread::Create(this, *FString::Printf(TEXT("AudioMixerRenderThread(%d)"), AudioMixerTaskCounter.Increment()), 0, (EThreadPriority)SetRenderThreadPriorityCVar, RenderThreadAffinityCVar));
		check(AudioRenderThread.IsValid());
	}

	void IAudioMixerPlatformInterface::StopGeneratingAudio()
	{		
		SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_StopGeneratingAudio, FColor::Blue);

		// Stop the FRunnable thread

		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped)
		{
			AudioStreamInfo.StreamState = EAudioOutputStreamState::Stopping;
		}

		if (AudioRenderEvent != nullptr)
		{
			// Make sure the thread wakes up
			AudioRenderEvent->Trigger();
		}

		if (AudioRenderThread.IsValid())
		{
			{
				SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_StopGeneratingAudio_KillRenderThread, FColor::Blue);
				AudioRenderThread->Kill();
			}

			// WaitForCompletion will complete right away when single threaded, and AudioStreamInfo.StreamState will never be set to stopped
			if (FPlatformProcess::SupportsMultithreading())
			{
				check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);
			}

			AudioRenderThread.Reset();
		}
		
		AudioStreamInfo.StreamState = EAudioOutputStreamState::Stopped;

		if (AudioRenderEvent != nullptr)
		{
			FPlatformProcess::ReturnSynchEventToPool(AudioRenderEvent);
			AudioRenderEvent = nullptr;
		}

		if (AudioFadeEvent != nullptr)
		{
			FPlatformProcess::ReturnSynchEventToPool(AudioFadeEvent);
			AudioFadeEvent = nullptr;
		}

		bIsGeneratingAudio = false;
	}

	void IAudioMixerPlatformInterface::SignalStopGeneratingAudio()
	{
		SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_SignalStopGeneratingAudio, FColor::Blue);

		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped)
		{
			// If no render thread is running, nothing will transition Stopping → Stopped,
			// and TryFinishStopGeneratingAudio() will poll the full timeout. Set Stopped
			// directly so cleanup completes the next tick.
			AudioStreamInfo.StreamState = bIsGeneratingAudio
				? EAudioOutputStreamState::Stopping
				: EAudioOutputStreamState::Stopped;
		}

		if (AudioRenderEvent != nullptr)
		{
			// Make sure the thread wakes up so it can observe the Stopping state
			AudioRenderEvent->Trigger();
		}

		StopGeneratingAudioSignalTime = FPlatformTime::Seconds();

		UE_LOGF(LogAudioMixer, Display, "SignalStopGeneratingAudio() called, InstanceID=%d", InstanceID);
	}

	bool IAudioMixerPlatformInterface::TryFinishStopGeneratingAudio()
	{
		SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_TryFinishStopGeneratingAudio, FColor::Blue);

		// Check if the render thread has exited on its own.
		// RunInternal() sets StreamState = Stopped before returning.
		const bool bRenderThreadStopped = (AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);

		if (!bRenderThreadStopped)
		{
			// Check for timeout - if the render thread hasn't stopped in time, we abandon it
			// rather than calling Kill(true) which would block indefinitely (WaitForSingleObject
			// with INFINITE timeout) and recreate the same deadlock we're trying to avoid.
			const double TimeoutSeconds = static_cast<double>(FMath::Clamp<int32>(OverrunTimeoutCVar, 500, 5000)) / 1000.0;
			const double ElapsedSeconds = FPlatformTime::Seconds() - StopGeneratingAudioSignalTime;
			if (ElapsedSeconds < TimeoutSeconds)
			{
				return false; // Still waiting for the render thread to exit
			}

			UE_LOGF(LogAudioMixer, Error,
				"TryFinishStopGeneratingAudio() timed out after %.2fs waiting for render thread to stop. " "Abandoning render thread to avoid deadlock. The thread may still be blocked in Task.Wait(). InstanceID=%d",
				ElapsedSeconds, InstanceID);
		}

		if (AudioRenderThread.IsValid())
		{
			if (bRenderThreadStopped)
			{
				// Thread has exited - Kill returns immediately, just cleans up the handle.
				SCOPED_NAMED_EVENT(IAudioMixerPlatformInterface_TryFinishStopGeneratingAudio_KillRenderThread, FColor::Blue);
				AudioRenderThread->Kill(true);
			}
			else
			{
				// Thread is still running. We cannot call Kill(true) as it would block indefinitely.
				// We cannot call Kill(false) as it closes the thread handle while the thread is still
				// running, causing use-after-free when the thread eventually exits.
				// Release ownership without destroying - the thread will leak but the process won't deadlock.
				(void)AudioRenderThread.Release();
				UE_LOGF(LogAudioMixer, Warning, "TryFinishStopGeneratingAudio() leaked render thread handle to avoid deadlock. InstanceID=%d", InstanceID);
			}

			AudioRenderThread.Reset();
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Stopped;

		if (AudioRenderEvent != nullptr)
		{
			if (bRenderThreadStopped)
			{
				FPlatformProcess::ReturnSynchEventToPool(AudioRenderEvent);
			}
			// If the render thread is still running, it may still reference AudioRenderEvent.
			// Don't return it to the pool - leak it to avoid use-after-free.
			AudioRenderEvent = nullptr;
		}

		if (AudioFadeEvent != nullptr)
		{
			if (bRenderThreadStopped)
			{
				FPlatformProcess::ReturnSynchEventToPool(AudioFadeEvent);
			}
			// Same as above - leak rather than return to pool if thread is still running.
			AudioFadeEvent = nullptr;
		}

		bIsGeneratingAudio = false;
		return true;
	}

	void IAudioMixerPlatformInterface::Tick()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		// In single-threaded mode, we simply render buffers until we run out of space
		// The single-thread audio backend will consume these rendered buffers when they need to
		if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running && bIsDeviceInitialized)
		{
			// Render mixed buffers till our queued buffers are filled up
			while (OutputBuffer.MixNextBuffer())
			{
			}
		}
	}

	uint32 IAudioMixerPlatformInterface::MainAudioDeviceRun()
	{
		return RunInternal();
	}

	uint32 IAudioMixerPlatformInterface::RunInternal()
	{
		UE_LOGF(LogAudioMixer, Display, "Starting AudioMixerPlatformInterface::RunInternal(), InstanceID=%d", InstanceID);

		// Lets prime and submit the first buffer (which is going to be the buffer underrun buffer)
		int32 NumSamplesPopped;
		TArrayView<const uint8> AudioToSubmit = OutputBuffer.PopBufferData(NumSamplesPopped);

		SubmitBuffer(AudioToSubmit.GetData());

		OutputBuffer.MixNextBuffer();

		while (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopping)
		{
			// Render mixed buffers till our queued buffers are filled up
			while (bIsDeviceInitialized && OutputBuffer.MixNextBuffer())
			{
			}

			// Bounds check the timeout for our audio render event, waiting indefinitely if a debugger is attached
			const uint32 CurrentWaitTimeout = FPlatformMisc::IsDebuggerPresent() ? MAX_uint32 : FMath::Clamp<uint32>(OverrunTimeoutCVar, 500, 5000);

			// Now wait for a buffer to be consumed, which will bump up the read index.
			const double WaitStartTime = FPlatformTime::Seconds();
			if (AudioRenderEvent && !AudioRenderEvent->Wait(CurrentWaitTimeout))
			{
				const float TimeWaited = FPlatformTime::Seconds() - WaitStartTime;

				// Only request a recovery swap if we're actually running. If we're already
				// mid-swap (SwappingDevice) or being torn down (Stopping/Stopped), the swap
				// orchestrator owns recovery from here. Issuing RequestDeviceSwap from this
				// thread in those states is a lock+Kill deadlock: this thread blocks on
				// DeviceSwapCriticalSection while the orchestrator holds it and is in
				// AudioRenderThread->Kill() waiting for THIS thread to exit.
				//
				// StreamState is std::atomic so the read is safe without locking. A Running
				// -> SwappingDevice transition we miss here gets caught on the next iteration
				// of the outer loop's state check, so no correctness issue.
				const EAudioOutputStreamState::Type StreamState = AudioStreamInfo.StreamState;
				if (StreamState == EAudioOutputStreamState::Running)
				{
					// if we reached this block, we timed out, and should attempt to
					// bail on our current device.
					RequestDeviceSwap(TEXT(""), /* force */true, TEXT("AudioMixerPlatformInterface. Timeout waiting for h/w."));
					UE_LOGF(LogAudioMixer, Warning, "AudioMixerPlatformInterface Timeout [%2.f Seconds] waiting for h/w. InstanceID=%d", TimeWaited, InstanceID);
				}
				else
				{
					UE_LOGF(LogAudioMixer, Display, "AudioMixerPlatformInterface Timeout [%2.f Seconds] waiting for h/w while StreamState=%d; skipping recovery swap (orchestrator owns teardown/swap). InstanceID=%d", TimeWaited, (int32)StreamState, InstanceID);
				}
			}
		}

		OpenStreamParams.AudioMixer->OnAudioStreamShutdown();

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Stopped;
		return 0;
	}

	uint32 IAudioMixerPlatformInterface::Run()
	{	
		LLM_SCOPE(ELLMTag::AudioMixer);

		FScopedFTZFloatMode FTZ;

		uint32 ReturnVal = 0;
		FMemory::SetupTLSCachesOnCurrentThread();

		// Call different functions depending on if it's the "main" audio mixer instance. Helps debugging callstacks.
		if (AudioStreamInfo.AudioMixer->IsMainAudioMixer())
		{
			ReturnVal = MainAudioDeviceRun();
		}
		else
		{
			ReturnVal = RunInternal();
		}

		FMemory::ClearAndDisableTLSCachesOnCurrentThread();
		return ReturnVal;
	}

	bool IAudioMixerPlatformInterface::GetChannelTypeAtIndex(const int32 Index, EAudioMixerChannel::Type& OutType)
	{
		return ChannelUtils::GetChannelTypeAtIndex(Index, OutType);
	}

	bool IAudioMixer::ShouldUseThreadedDeviceSwap()
	{
#if PLATFORM_WINDOWS 
		return bUseThreadedDeviceSwapCVar != 0;
#else //PLATFORM_WINDOWS
		return false;
#endif //PLATFORM_WINDOWS
	}

	bool IAudioMixer::ShouldUseDeviceInfoCache()
	{		
#if PLATFORM_WINDOWS 
		return bUseAudioDeviceInfoCacheCVar != 0;
#else //PLATFORM_WINDOWS
		return false;
#endif //PLATFORM_WINDOWS
	}
	
	bool IAudioMixer::ShouldRecycleThreads()
	{
		return bRecycleThreadsCVar != 0;
	}

	FAudioMixerPlatformSwappable::FAudioMixerPlatformSwappable()
	{
	}

	bool FAudioMixerPlatformSwappable::RequestDeviceSwap(const FString& DeviceID, const bool bInForce, const TCHAR* InReason)
	{
		// This critical section protects that device swap context below.
		FScopeLock Lock(&DeviceSwapCriticalSection);

		// If the render thread is being stopped for a device swap, we cannot set StreamState to
		// SwappingDevice (it would overwrite the Stopping state the render thread needs to see).
		// Defer the request so it can be replayed after the current swap completes.
		if (bWaitingForRenderThreadStop)
		{
			if (bInForce)
			{
				DeferredDeviceSwapId = DeviceID;
				DeferredDeviceSwapReason = InReason ? InReason : TEXT("None specified");
				UE_LOGF(LogAudioMixer, Display, "Deferring force device swap request to '%ls' (reason: '%ls') until render thread stops.",
					!DeviceID.IsEmpty() ? *DeviceID : TEXT("[System Default]"),
					*DeferredDeviceSwapReason
				);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Display, "Ignoring non-force device swap request, waiting for render thread to stop from previous swap.");
			}
			return false;
		}

		if (AllowDeviceSwap(bInForce))
		{
			UE_LOGF(LogAudioMixer, Display, "Attempt to swap audio render device to new device: '%ls', because: '%ls', force=%d",
				!DeviceID.IsEmpty() ? *DeviceID : TEXT("[System Default]"),
				InReason ? InReason : TEXT("None specified"),
				(int32)bInForce
			);

			// Set up the context for this device swap
			if (InitializeDeviceSwapContext(DeviceID, InReason))
			{
				// Set the flag indicating we wish to begin a device swap
				AudioStreamInfo.StreamState = EAudioOutputStreamState::SwappingDevice;
				return true;
			}
		}
		else
		{
			UE_LOGF(LogAudioMixer, Display, "NOT-ALLOWING attempt to swap audio render device to new device: '%ls', because: '%ls', force=%d [normal if occurring while suspended on consoles or if force=false and another swap is in-flight]",
				!DeviceID.IsEmpty() ? *DeviceID : TEXT("[System Default]"),
				InReason ? InReason : TEXT("None specified"),
				(int32)bInForce
			);
		}

		return false;
	}

	bool FAudioMixerPlatformSwappable::CheckAudioDeviceChange()
	{
		SCOPED_NAMED_EVENT(FAudioMixerPlatformSwappable_CheckAudioDeviceChange, FColor::Blue);

		// Use threaded version? (It also requires the info cache).
		if (IAudioMixer::ShouldUseThreadedDeviceSwap() && ShouldUseDeviceInfoCache())
		{
			return CheckThreadedDeviceSwap();
		}
		
		if (AudioStreamInfo.StreamState == EAudioOutputStreamState::SwappingDevice)
		{
			return MoveAudioStreamToNewAudioDevice();
		}

		return false;
	}
	
	bool FAudioMixerPlatformSwappable::AllowDeviceSwap(const bool bInForceSwap)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double TimeDelta = CurrentTime - LastDeviceSwapTime;

		LastDeviceSwapTime = CurrentTime;

		if (!IsInitialized())
		{
			UE_LOGF(LogAudioMixer, Display, "Unable to device swap until backend is initialized.");
			return false;
		}

		// If the device is not running or if a device swap is already in progress, don't "double-trigger" a swap unless
		// bInForceSwap is true, in which case we'll replace the existing request.
		if (ActiveDeviceSwap.IsValid() && !bInForceSwap)
		{
			UE_LOGF(LogAudioMixer, Display, "Ignoring device swap request, swap already requested. bInForceSwap: %d", bInForceSwap);
			return false;
		}

		// Only device swap if the audio device is running, stopped or mid-swap
		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Running && 
			AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped &&
			AudioStreamInfo.StreamState != EAudioOutputStreamState::SwappingDevice)
		{
			UE_LOGF(LogAudioMixer, Display, "Ignoring device swap request, AudioStreamInfo.StreamState: %d", AudioStreamInfo.StreamState.load());
			return false;
		}

		// Some devices spam device swap notifications, so we want to rate-limit them to prevent double/triple triggering.
		static constexpr double MinSwapTimeSeconds = 10.0 / 1000.0;
		if (TimeDelta <= MinSwapTimeSeconds && !bInForceSwap)
		{
			UE_LOGF(LogAudioMixer, Display, "IAudioMixerPlatformInterface::AllowDeviceSwap ignoring device swap due to rate-limit; LastDeviceSwapTime: %f CurrentTime: %f", LastDeviceSwapTime, CurrentTime);
			return false;
		}

		return true;
	}
	
	bool FAudioMixerPlatformSwappable::CheckThreadedDeviceSwap()
	{
		bool bDidStopGeneratingAudio = false;
		SCOPED_NAMED_EVENT(FAudioMixerPlatformSwappable_CheckThreadedDeviceSwap, FColor::Blue);

		// Phase 2: If we previously signaled the render thread to stop, check if it's done.
		// This is checked first so that the swap completes as soon as the render thread exits.
		if (bWaitingForRenderThreadStop)
		{
			FScopeLock Lock(&DeviceSwapCriticalSection);

			if (!TryFinishStopGeneratingAudio())
			{
				return false; // Still waiting for render thread to exit
			}

			// Render thread has stopped — proceed with swap completion
			bWaitingForRenderThreadStop = false;
			bDidStopGeneratingAudio = true;

			if (!PostDeviceSwap())
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerPlatformSwappable::CheckThreadedDeviceSwap PostDeviceSwap() failed");
			}

			// If a force swap was requested while we were waiting, replay it now that
			// the render thread is stopped and the previous swap is complete.
			if (DeferredDeviceSwapId.IsSet())
			{
				FString ReplayDeviceId = MoveTemp(DeferredDeviceSwapId.GetValue());
				FString ReplayReason = MoveTemp(DeferredDeviceSwapReason);
				DeferredDeviceSwapId.Reset();

				UE_LOGF(LogAudioMixer, Display, "Replaying deferred device swap to '%ls' (reason: '%ls')",
					!ReplayDeviceId.IsEmpty() ? *ReplayDeviceId : TEXT("[System Default]"),
					*ReplayReason);

				// This is safe now — bWaitingForRenderThreadStop is false and StreamState is Stopped,
				// so RequestDeviceSwap can set StreamState to SwappingDevice without conflict.
				RequestDeviceSwap(ReplayDeviceId, /*bInForce*/ true, *ReplayReason);
			}

			return bDidStopGeneratingAudio;
		}

		// Phase 1: Check-lock-check pattern for starting or completing async swap.
		// Because this is called every tick, utilize the check-lock-check pattern to avoid
		// unnecessary locking.
		if (AudioStreamInfo.StreamState == EAudioOutputStreamState::SwappingDevice || ActiveDeviceSwap.IsValid())
		{
			// Lock and verify this device is still running.
			FScopeLock Lock(&DeviceSwapCriticalSection);

			// Check the stream state again now that the lock is acquired in case the device is being closed down
			EAudioOutputStreamState::Type StreamState = AudioStreamInfo.StreamState;

			// Start a job?
			if (StreamState == EAudioOutputStreamState::SwappingDevice && !ActiveDeviceSwap.IsValid())
			{
				SCOPED_NAMED_EVENT(FAudioMixerPlatformSwappable_CheckThreadedDeviceSwap_StartAsyncSwap, FColor::Blue);

				if (!PreDeviceSwap())
				{
					UE_LOGF(LogAudioMixer, Display, "FAudioMixerPlatformSwappable::CheckThreadedDeviceSwap PreDeviceSwap() failed");
					return bDidStopGeneratingAudio;
				}

				EnqueueAsyncDeviceSwap();
			}
			else if (ActiveDeviceSwap.IsReady())
			{
				// Async swap finished — signal the render thread to stop without blocking.
				// We return false this tick so that the audio thread releases back to the task pool,
				// which breaks a potential deadlock where StopGeneratingAudio()->Kill() blocks on the
				// render thread while the render thread is blocked in Task.Wait() waiting for task
				// workers that can't run because this thread is blocked.
				SCOPED_NAMED_EVENT(FAudioMixerPlatformSwappable_CheckThreadedDeviceSwap_SignalStop, FColor::Blue);

				if (bIsUsingNullDevice)
				{
					StopRunningNullDevice();
				}

				SignalStopGeneratingAudio();
				bWaitingForRenderThreadStop = true;
			}
		}

		return bDidStopGeneratingAudio;
	}
	
	bool FAudioMixerPlatformSwappable::MoveAudioStreamToNewAudioDevice()
	{
		SCOPED_NAMED_EVENT(FAudioMixerPlatformSwappable_MoveAudioStreamToNewAudioDevice, FColor::Blue);

		FScopeLock Lock(&DeviceSwapCriticalSection);
		bool bDidStopGeneratingAudio = false;

		if (!PreDeviceSwap())
		{
			UE_LOGF(LogAudioMixer, Warning, "FAudioMixerPlatformSwappable::MoveAudioStreamToNewAudioDevice PreDeviceSwap() failed");
			return bDidStopGeneratingAudio;
		}

		// Swap devices
		SynchronousDeviceSwap();

		const FDeviceSwapResult* DeviceSwapResult = GetDeviceSwapResult();

		// Stop null device if currently running
		if (bIsUsingNullDevice)
		{
			StopRunningNullDevice();
		}

		// Device swaps require reinitialization of output buffers to handle
		// different channel formats. Stop generating audio to protect against
		// accessing the OutputBuffer.
		StopGeneratingAudio();
		bDidStopGeneratingAudio = true;
		
		if (!PostDeviceSwap())
		{
			UE_LOGF(LogAudioMixer, Warning, "FAudioMixerPlatformSwappable::MoveAudioStreamToNewAudioDevice CompleteDeviceSwap() failed");
			return bDidStopGeneratingAudio;
		}
		
		return bDidStopGeneratingAudio; 
	}

	void FAudioMixerPlatformSwappable::ResumePlaybackOnNewDevice()
	{
		SCOPED_NAMED_EVENT(FAudioMixerPlatformSwappable_ResumePlaybackOnNewDevice, FColor::Blue);
		UE_LOGF(LogAudioMixer, Display, "FAudioMixerPlatformSwappable::ResumePlaybackOnNewDevice - resuming audio on new device: '%ls'", *AudioStreamInfo.DeviceInfo.Name);

		int32 NumSamplesPopped = 0;
		TArrayView<const uint8> PoppedAudio = OutputBuffer.PopBufferData(NumSamplesPopped);
		SubmitBuffer(PoppedAudio.GetData());

		check(OpenStreamParams.NumFrames * AudioStreamInfo.DeviceInfo.NumChannels == OutputBuffer.GetNumSamples());

		StartAudioStream();
		AudioRenderEvent->Trigger();
	}
	
}

FAudioPlatformSettings FAudioPlatformSettings::GetPlatformSettings(const TCHAR* PlatformSettingsConfigFile)
{
	FAudioPlatformSettings Settings;

	FString TempString;

	if (GConfig->GetString(PlatformSettingsConfigFile, TEXT("AudioSampleRate"), TempString, GEngineIni))
	{
		Settings.SampleRate = FMath::Max(FCString::Atoi(*TempString), 8000);
	}

	if (GConfig->GetString(PlatformSettingsConfigFile, TEXT("AudioCallbackBufferFrameSize"), TempString, GEngineIni))
	{
		Settings.CallbackBufferFrameSize = FMath::Max(FCString::Atoi(*TempString), 240);
	}

	if (GConfig->GetString(PlatformSettingsConfigFile, TEXT("AudioNumBuffersToEnqueue"), TempString, GEngineIni))
	{
		Settings.NumBuffers = FMath::Max(FCString::Atoi(*TempString), 1);
	}

	if (GConfig->GetString(PlatformSettingsConfigFile, TEXT("AudioMaxChannels"), TempString, GEngineIni))
	{
		Settings.MaxChannels = FMath::Max(FCString::Atoi(*TempString), 0);
	}

	if (GConfig->GetString(PlatformSettingsConfigFile, TEXT("AudioNumSourceWorkers"), TempString, GEngineIni))
	{
		Settings.NumSourceWorkers = FMath::Max(FCString::Atoi(*TempString), 0);
	}

	return Settings;
}
