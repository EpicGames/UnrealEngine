// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerCoreModule.h"
#include "Modules/ModuleManager.h"
#include "AudioMixerLog.h"

DEFINE_LOG_CATEGORY(LogAudioMixer);
DEFINE_LOG_CATEGORY(LogAudioMixerDebug);

class FAudioMixerCoreModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModuleChecked(TEXT("SignalProcessing"));
	}
};

IMPLEMENT_MODULE(FAudioMixerCoreModule, AudioMixerCore);
