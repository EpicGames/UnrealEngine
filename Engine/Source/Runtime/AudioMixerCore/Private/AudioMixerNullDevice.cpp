// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerNullDevice.h"
#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/Event.h"
#include "AudioMixerLog.h"
#include "Misc/ScopeLock.h"

namespace Audio
{
	uint32 FMixerNullCallback::Run()
	{
		//
		// To simulate an audio device requesting for more audio, we sleep between callbacks.
		// The problem with this is that OS/Kernel Sleep is not accurate. It will always be slightly higher than requested,
		// which means that audio will be generated slightly slower than the stated sample rate.
		// To correct this, we keep track of the real time passed, and adjust the sleep time accordingly so the audio clock
		// stays as close to the real time clock as possible.

		double AudioClock = FPlatformTime::Seconds();

		check(SleepEvent);

		float SleepTime = CallbackTime; 
		
		while (!bShouldShutdown)
		{
			// Wait here to be woken up.
			if (WakeupEvent)
			{
				WakeupEvent->Wait(MAX_uint32);
				WakeupEvent->Reset();
				
				UE_CLOGF(!bShouldShutdown && !bShouldRecyle, LogAudioMixer, Display, "FMixerNullCallback: Simulating a h/w device callback at [%dms], ThreadID=%u",  (int32)(CallbackTime * 1000.f), CallbackThread->GetThreadID() );

				// Reset our time differential.
				AudioClock = FPlatformTime::Seconds();
				SleepTime = CallbackTime;
			}

			// Simulate a null h/w device as long as we've have been asked to shutdown/recycle
			while (!bShouldRecyle && !bShouldShutdown)
			{
				SCOPED_NAMED_EVENT(FMixerNullCallback_Run_Working, FColor::Blue);

				Callback();

				// Clamp to Maximum of 200ms.
				float SleepTimeClampedMs = FMath::Clamp<float>(SleepTime * 1000.f, 0.f, 200.f);

				// Wait with a timeout of our sleep time. Triggering the event will leave the wait early.
				bool bTriggered = SleepEvent->Wait((int32)SleepTimeClampedMs);
				SleepEvent->Reset();

				AudioClock += CallbackTime;
				double RealClock = FPlatformTime::Seconds();
				double AudioVsReal = RealClock - AudioClock;

				// For the next sleep, we adjust the sleep duration to try and keep the audio clock as close
				// to the real time clock as possible
				SleepTime = CallbackTime - AudioVsReal;

#if !NO_LOGGING
				// Warn if there's any crazy deltas (limit to every 30s).
				if (RealClock - LastLog > 30.f)
				{
					if (FMath::Abs(SleepTime) > 0.2f)
					{
						UE_LOGF(LogAudioMixer, Warning, "FMixerNullCallback: Large time delta between simulated audio clock and realtime [%dms], ThreadID=%u", (int32)(SleepTime * 1000.f), CallbackThread->GetThreadID());
						LastLog = RealClock;
					}
				}
#endif //!NO_LOGGING
			}
		}
		return 0;
	}

	FMixerNullCallback::FMixerNullCallback(float InBufferDuration, TFunction<void()> InCallback, EThreadPriority ThreadPriority, bool bStartPaused)
		: Callback(InCallback)
		, CallbackTime(InBufferDuration)
		, bShouldShutdown(false)
		, SleepEvent(FPlatformProcess::GetSynchEventFromPool(true))
		, WakeupEvent(FPlatformProcess::GetSynchEventFromPool(true))
	{
		check(SleepEvent);
		check(WakeupEvent);

		// Make sure we're in a waitable start on startup.
		SleepEvent->Reset();

		CallbackThread.Reset(FRunnableThread::Create(this, TEXT("AudioMixerNullCallbackThread"), 0, ThreadPriority, FPlatformAffinity::GetAudioRenderThreadMask()));

		// If we are marked to pause on startup, make sure the event is in a waitable state.
		if (bStartPaused)
		{
			WakeupEvent->Reset();
		}
		else
		{
			WakeupEvent->Trigger();
		}		
	}
		
	void FMixerNullCallback::Stop()
	{
		SCOPED_NAMED_EVENT(FMixerNullCallback_Stop, FColor::Blue);

		// Flag loop to exit 
		bShouldShutdown = true;
	
		// If we're waiting for wakeup event, trigger that to bail the loop.
		if (WakeupEvent)
		{
			WakeupEvent->Trigger();
		}

		if (SleepEvent)
		{
			// Exit any sleep we're inside.
			SleepEvent->Trigger();

			if (CallbackThread.IsValid())
			{
				// Wait to continue, before deleteing the events.
				CallbackThread->WaitForCompletion();
			}	

			FPlatformProcess::ReturnSynchEventToPool(SleepEvent);
			SleepEvent = nullptr;
		}

		if (WakeupEvent)
		{
			FPlatformProcess::ReturnSynchEventToPool(WakeupEvent);
			WakeupEvent = nullptr;
		}
	}

	void FMixerNullCallback::Resume(const TFunction<void()>& InCallback, float InBufferDuration)
	{
		if (WakeupEvent)
		{
			// Copy all the new state and trigger the start event.
			// Note we do this without a lock, assuming we're waiting on the wakeup event.
			Callback = InCallback;
			CallbackTime = InBufferDuration;
			bShouldRecyle = false;
			FPlatformMisc::MemoryBarrier();
			WakeupEvent->Trigger();
		}
	}

	void FMixerNullCallback::Pause()
	{
		// Flag that we should recycle the thread, causing us to bail the inner loop wait on the start event.
		bShouldRecyle = true;
		if (SleepEvent)
		{
			// Early out the sleep.
			SleepEvent->Trigger();
		}
	}
}
