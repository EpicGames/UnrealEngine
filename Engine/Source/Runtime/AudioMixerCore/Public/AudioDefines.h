// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"


/**
 * Maximum number of channels that can be set using the ini setting
 */
#define MAX_AUDIOCHANNELS				64


/**
 * Length of sound in seconds to be considered as looping forever
 */
#define INDEFINITELY_LOOPING_DURATION	10000.0f


/**
 * Some defaults to help cross platform consistency
 */
#define SPEAKER_COUNT					6

#define DEFAULT_LOW_FREQUENCY			600.0f
#define DEFAULT_MID_FREQUENCY			1000.0f
#define DEFAULT_HIGH_FREQUENCY			2000.0f

#define MAX_VOLUME						4.0f
#define MIN_PITCH						0.4f
#define MAX_PITCH						2.0f

#define MIN_VOLUME_LINEAR				SMALL_NUMBER
#define MIN_VOLUME_DECIBELS				-160.f

#define MIN_SOUND_PRIORITY				0.0f
#define MAX_SOUND_PRIORITY				100.0f

#define DEFAULT_SUBTITLE_PRIORITY		10000.0f


/**
 * Some filters don't work properly with extreme values, so these are the limits 
 */
#define MIN_FILTER_GAIN					0.126f
#define MAX_FILTER_GAIN					7.94f

#define MIN_FILTER_FREQUENCY			20.0f
#define MAX_FILTER_FREQUENCY			20000.0f

#define MIN_FILTER_BANDWIDTH			0.1f
#define MAX_FILTER_BANDWIDTH			2.0f


/**
 * Debugger is Available on non-shipping builds
 */
#define ENABLE_AUDIO_DEBUG !UE_BUILD_SHIPPING


/** Common Audio namespace Type Definitions/Identifiers */
namespace Audio
{
	/**
	 * Typed identifier for Audio Device Id
	 */
	using FDeviceId = uint32;


	// Magic numbers for mono upmix method
	inline constexpr float MonoUpmixFullVolume = 1.0f;
	inline constexpr float MonoUpmixEqualPower = 0.707f;
	inline constexpr float MonoUpmixLinear = 0.5f;

}