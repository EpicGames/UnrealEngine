// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "AudioMixerChannel.h"
#include "AudioMixerLog.h"
#include "AudioMixerNullDevice.h"
#include "AudioMixerTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/ParamInterpolator.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformMath.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"
#include "Misc/SingleThreadRunnable.h"
#include "Modules/ModuleInterface.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"

#define UE_API AUDIOMIXERCORE_API

class FEvent;
class FRunnableThread;
class FThreadSafeCounter;
namespace Audio { class FMixerNullCallback; }

// defines used for AudioMixer.h
#define AUDIO_PLATFORM_LOG_ONCE(INFO, VERBOSITY)	(AudioMixerPlatformLogOnce(INFO, FString(__FILE__), __LINE__, ELogVerbosity::VERBOSITY))
#define AUDIO_PLATFORM_ERROR(INFO)					(AudioMixerPlatformLogOnce(INFO, FString(__FILE__), __LINE__, ELogVerbosity::Error))

#ifndef AUDIO_MIXER_ENABLE_DEBUG_MODE
// This define enables a bunch of more expensive debug checks and logging capabilities that are intended to be off most of the time even in debug builds of game/editor.
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define AUDIO_MIXER_ENABLE_DEBUG_MODE 0
#else
#define AUDIO_MIXER_ENABLE_DEBUG_MODE 1
#endif
#endif


// Enable debug checking for audio mixer

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
#define AUDIO_MIXER_CHECK(expr) ensure(expr)
#define AUDIO_MIXER_CHECK_GAME_THREAD(_MixerDevice)			(_MixerDevice->CheckAudioThread())
#define AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(_MixerDevice)	(_MixerDevice->CheckAudioRenderingThread())
#else
#define AUDIO_MIXER_CHECK(expr)
#define AUDIO_MIXER_CHECK_GAME_THREAD(_MixerDevice)
#define AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(_MixerDevice)
#endif

#define AUDIO_MIXER_MAX_OUTPUT_CHANNELS				8			// Max number of speakers/channels supported (7.1)

#define AUDIO_MIXER_DEFAULT_DEVICE_INDEX			INDEX_NONE

// Cycle stats for audio mixer
DECLARE_STATS_GROUP(TEXT("AudioMixer"), STATGROUP_AudioMixer, STATCAT_Advanced);

// Tracks the time for the full render block 
DECLARE_CYCLE_STAT_EXTERN(TEXT("Render Audio"), STAT_AudioMixerRenderAudio, STATGROUP_AudioMixer, AUDIOMIXERCORE_API);


class FSoundWaveData;
class FSoundWaveProxy;
class ICompressedAudioInfo;
class USoundWave;

using FSoundWaveProxyPtr = TSharedPtr<FSoundWaveProxy, ESPMode::ThreadSafe>;
using FSoundWavePtr = TSharedPtr<FSoundWaveData, ESPMode::ThreadSafe>;


namespace Audio
{
   	/** Structure to hold platform device information **/
	struct FAudioPlatformDeviceInfo
	{
		/** The name of the audio device */
		FString Name;

		/** ID of the device. */
		FString DeviceId;

		/** The number of channels supported by the audio device */
		int32 NumChannels;

		/** The number of channels above the base stereo or 7.1 channels supported by the audio device */
		int32 NumDirectOutChannels;

		/** The sample rate of the audio device */
		int32 SampleRate;

		/** The data format of the audio stream */
		EAudioMixerStreamDataFormat::Type Format;

		/** The output channel array of the audio device */
		TArray<EAudioMixerChannel::Type> OutputChannelArray;

		/** Whether or not this device is the system default */
		uint8 bIsSystemDefault : 1;

		FAudioPlatformDeviceInfo()
		{
			Reset();
		}

		void Reset()
		{
			Name = TEXT("Unknown");
			DeviceId = TEXT("Unknown");
			NumChannels = 0;
			NumDirectOutChannels = 0;
			SampleRate = 0;
			Format = EAudioMixerStreamDataFormat::Unknown;
			OutputChannelArray.Reset();
			bIsSystemDefault = false;
		}

	};

	/** Platform independent audio mixer interface. */
	class IAudioMixer
	{
	public:
		/** Callback to generate a new audio stream buffer. */
		virtual bool OnProcessAudioStream(FAlignedFloatBuffer& OutputBuffer) = 0;

		/** Called when audio render thread stream is shutting down. Last function called. Allows cleanup on render thread. */
		virtual void OnAudioStreamShutdown() = 0;

		bool IsMainAudioMixer() const { return bIsMainAudioMixer; }

		/** Called by AudioMixer to see if we should do a multithreaded device swap */
		AUDIOMIXERCORE_API static bool ShouldUseThreadedDeviceSwap();

		/** Called by AudioMixer to see if it should reycle the threads: */
		AUDIOMIXERCORE_API static bool ShouldRecycleThreads();

		/** Called by AudioMixer if it should use Cache for DeviceInfo Enumeration */
		UE_DEPRECATED(5.6, "ShouldUseDeviceInfoCache functionality moved to IAudioMixerPlatformInterface.")
		AUDIOMIXERCORE_API static bool ShouldUseDeviceInfoCache();


	protected:

		IAudioMixer() 
		: bIsMainAudioMixer(false) 
		{}

		bool bIsMainAudioMixer;
	};

	enum class EDeviceEndpointType { Unknown, Render, Capture };

	enum class EAudioDeviceRole
	{
		Console,
		Multimedia,
		Communications,

		COUNT,
	};

	enum class EAudioDeviceState
	{
		Active,
		Disabled,
		NotPresent,
		Unplugged,

		COUNT,
	};

	// Interface for Caching Device Info.
	class IAudioPlatformDeviceInfoCache
	{
	public:
		// Pure Interface. 
		virtual ~IAudioPlatformDeviceInfoCache() = default;
			
		virtual FName GetDefaultOutputDevice(EAudioDeviceRole InRole) const = 0;
		virtual TOptional<FAudioPlatformDeviceInfo> FindActiveOutputDevice(FName InDeviceID) const = 0;
		virtual TArray<FAudioPlatformDeviceInfo> GetAllActiveOutputDevices() const = 0;
		virtual TOptional<FAudioPlatformDeviceInfo> FindDefaultOutputDevice() const = 0;

		/** Determines if the given device Id is that of an aggregate device which uses hardware Ids as device Ids. */
		virtual bool IsAggregateHardwareDeviceId(const FName InDeviceId) const = 0;

		/** Returns an array of logical audio devices which, when combined, form an aggregate device. */
		virtual TArray<FAudioPlatformDeviceInfo> GetLogicalAggregateDevices(const FName InHardwareId, const EDeviceEndpointType InEndpointType) const = 0;

		/** Refresh the data in the cache */
		virtual void Refresh() = 0;

		/** Mark a device inactive after a caller observed it unreachable at the OS level. */
		virtual void InvalidateDevice(FName InDeviceId) = 0;
	};

	/** Defines parameters needed for opening a new audio stream to device. */
	struct FAudioMixerOpenStreamParams
	{
		/** The audio device index to open. */
		uint32 OutputDeviceIndex;

		/** The number of desired audio frames in audio callback. */
		uint32 NumFrames;
		
		/** The number of queued buffers to use for the stream. */
		int32 NumBuffers;

		/** Owning platform independent audio mixer ptr.*/
		IAudioMixer* AudioMixer;
		
		/** The desired sample rate */
		uint32 SampleRate;

		/* The maximum number of sources we will try to decode or playback at once. */
		int32 MaxSources;

		/** Whether or not to try and restore audio to this stream if the audio device is removed (and the device becomes available again). */
		bool bRestoreIfRemoved;

		/** If true, use the system default audio device. If False, AudioDeviceId must contain a valid device Id. */
		bool bUseSystemAudioDevice;

		/** The device id of an audio output device. May be empty if bUseSystemAudioDevice is true. */
		FString AudioDeviceId;

		FAudioMixerOpenStreamParams()
			: OutputDeviceIndex(INDEX_NONE)
			, NumFrames(1024)
			, NumBuffers(1)
			, AudioMixer(nullptr)
			, SampleRate(44100)
			, MaxSources(0)
			, bRestoreIfRemoved(false)
			, bUseSystemAudioDevice(true)
		{}
	};

	struct FAudioOutputStreamInfo
	{
		/** The index of the output device for the audio stream. */
		uint32 OutputDeviceIndex;

		FAudioPlatformDeviceInfo DeviceInfo;

		/** The state of the output audio stream. */
		std::atomic<EAudioOutputStreamState::Type> StreamState;

		/** The callback to use for platform-independent layer. */
		IAudioMixer* AudioMixer;

		/** The number of queued buffers to use. */
		uint32 NumBuffers;

		/** Number of output frames */
		int32 NumOutputFrames;

		FAudioOutputStreamInfo()
		{
			Reset();
		}

		~FAudioOutputStreamInfo() = default;

		void Reset()
		{
			OutputDeviceIndex = 0;
			DeviceInfo.Reset();
			StreamState = EAudioOutputStreamState::Closed;
			AudioMixer = nullptr;
			NumBuffers = 2;
			NumOutputFrames = 0;
		}
	};

	/** Struct used to store render time analysis data. */
	struct FAudioRenderTimeAnalysis
	{
		double AvgRenderTime;
		double MaxRenderTime;
		double TotalRenderTime;
		double RenderTimeSinceLastLog;
		uint32 StartTime;
		double MaxSinceTick;
		uint64 RenderTimeCount;
		int32 RenderInstanceId;

		AUDIOMIXERCORE_API FAudioRenderTimeAnalysis();
		AUDIOMIXERCORE_API void Start();
		AUDIOMIXERCORE_API void End();
	};

	/** Class which wraps an output float buffer and handles conversion to device stream formats. */
	class FOutputBuffer
	{
	public:
		FOutputBuffer()
			: AudioMixer(nullptr)
			, DataFormat(EAudioMixerStreamDataFormat::Unknown)
		{}

		~FOutputBuffer() = default;
 
		/** Initialize the buffer with the given samples and output format. */
		AUDIOMIXERCORE_API void Init(IAudioMixer* InAudioMixer, const int32 InNumSamples, const int32 InNumBuffers, const EAudioMixerStreamDataFormat::Type InDataFormat);

		/** Gets the next mixed buffer from the audio mixer. Returns false if our buffer is already full. */
		AUDIOMIXERCORE_API bool MixNextBuffer();

		/** Gets the buffer data ptrs. Returns a TArrayView for the full buffer size requested, but in the case of an underrun, OutBytesPopped will be less that the size of the returned TArrayView. */
		AUDIOMIXERCORE_API TArrayView<const uint8> PopBufferData(int32& OutBytesPopped) const;

		/** Gets the number of frames of the buffer. */
		AUDIOMIXERCORE_API int32 GetNumSamples() const;

		/** Returns the format of the buffer. */
		EAudioMixerStreamDataFormat::Type GetFormat() const { return DataFormat; }


	private:
		IAudioMixer* AudioMixer;

		// Circular buffer used to buffer audio between the audio render thread and the platform interface thread.
		mutable Audio::TCircularAudioBuffer<uint8> CircularBuffer;
		
		// Buffer that we render audio to from the IAudioMixer instance associated with this output buffer.
		Audio::FAlignedFloatBuffer RenderBuffer;

		// Buffer read by the platform interface thread.
		mutable Audio::FAlignedByteBuffer PopBuffer;

		// For non-float situations, this buffer is used to convert RenderBuffer before pushing it to CircularBuffer.
		FAlignedByteBuffer FormattedBuffer;
 		EAudioMixerStreamDataFormat::Type DataFormat;

		static AUDIOMIXERCORE_API size_t GetSizeForDataFormat(EAudioMixerStreamDataFormat::Type InDataFormat);
		int32 CallCounterMixNextBuffer{ 0 };
	};

	/** Abstract interface for receiving audio device changed notifications */
	class IAudioMixerDeviceChangedListener
	{
	public:
		virtual ~IAudioMixerDeviceChangedListener() = default;

		struct FFormatChangedData
		{
			int32 NumChannels = 0;
			int32 SampleRate = 0;
			uint32 ChannelBitmask = 0;
		};

		enum class EDisconnectReason
		{
			DeviceRemoval,
			ServerShutdown,
			FormatChanged,
			SessionLogoff,
			SessionDisconnected,
			ExclusiveModeOverride
		};

		virtual void RegisterDeviceChangedListener() {}
		virtual void UnregisterDeviceChangedListener() {}
		virtual void OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) {}
		virtual void OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) {}
		virtual void OnDeviceAdded(const FString& DeviceId, bool bIsRenderDevice) {}
		virtual void OnDeviceRemoved(const FString& DeviceId, bool bIsRenderDevice) {}
		virtual void OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool bIsRenderDevice) {}
		virtual void OnFormatChanged(const FString& InDeviceId, const FFormatChangedData& InFormat) {}
		virtual void OnSessionDisconnect(EDisconnectReason InReason) {}
		
		virtual FString GetDeviceId() const { return FString(); }
	};

	
	struct FDeviceSwapContext
	{
		FDeviceSwapContext() = delete;
		FDeviceSwapContext(const FString& InRequestedDeviceID, const FString& InReason) :
			RequestedDeviceId(InRequestedDeviceID),
			DeviceSwapReason(InReason)
		{}
		virtual ~FDeviceSwapContext() = default;
		
		FString RequestedDeviceId;
		FString DeviceSwapReason;
		TOptional<FAudioPlatformDeviceInfo> NewDevice;
	};

	struct FDeviceSwapResult
	{
		virtual ~FDeviceSwapResult() = default;
		
		virtual bool IsNewDeviceReady() const { return false; }

		FAudioPlatformDeviceInfo DeviceInfo;
		FString SwapReason;
		double SuccessfulDurationMs = 0.0;
	};

	/** Abstract interface for mixer platform. */
	class IAudioMixerPlatformInterface : public FRunnable,
														public FSingleThreadRunnable,
														public IAudioMixerDeviceChangedListener
	{

	public: // Virtual functions
		
		/** Virtual destructor. */
		AUDIOMIXERCORE_API virtual ~IAudioMixerPlatformInterface();

		/** Returns the platform API name. */
		virtual FString GetPlatformApi() const = 0;

		/** Initialize the hardware. */
		virtual bool InitializeHardware() = 0;

		/** Check if audio device changed if applicable. Return true if audio device changed. */
		virtual bool CheckAudioDeviceChange() { return false; };

		/** Resumes playback on new audio device after device change. */
		virtual void ResumePlaybackOnNewDevice() {}

		/** Teardown the hardware. */
		virtual bool TeardownHardware() = 0;
		
		/** Is the hardware initialized. */
		virtual bool IsInitialized() const = 0;

		/** Returns the number of output devices. */
		virtual bool GetNumOutputDevices(uint32& OutNumOutputDevices) { OutNumOutputDevices = 1; return true; }

		/** Gets the device information of the given device index. */
		virtual bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) = 0;

		/**
		 * Returns the name of the currently used audio device.
		 */
		virtual FString GetCurrentDeviceName() const { return CurrentDeviceName; }

		/**
		 * Can be used to look up the current index for a given device name.
		 * On most platforms, this index may be invalidated if any devices are added or removed.
		 * Returns INDEX_NONE if no mapping is found
		 */
		AUDIOMIXERCORE_API virtual int32 GetIndexForDevice(const FString& InDeviceName);

		/** Gets the platform specific audio settings. */
		virtual FAudioPlatformSettings GetPlatformSettings() const = 0;

		/** Returns the default device index. */
		virtual bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const { OutDefaultDeviceIndex = 0; return true; }

		/** Opens up a new audio stream with the given parameters. */
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) = 0;

		/** Closes the audio stream (if it's open). */
		virtual bool CloseAudioStream() = 0;

		/** Starts the audio stream processing and generating audio. */
		virtual bool StartAudioStream() = 0;

		/** Stops the audio stream (but keeps the audio stream open). */
		virtual bool StopAudioStream() = 0;

		/** Resets the audio stream to use a new audio device with the given device ID (empty string means default). */
		UE_DEPRECATED(5.6, "Please use new version MoveAudioStreamToNewAudioDevice which takes no parameters.")
		virtual bool MoveAudioStreamToNewAudioDevice(const FString& InNewDeviceId) { return MoveAudioStreamToNewAudioDevice();  }
		virtual bool MoveAudioStreamToNewAudioDevice() { return true;  }

		/** Sends a command to swap which output device is being used */
		virtual bool RequestDeviceSwap(const FString& DeviceID, bool bInForce, const TCHAR* InReason = nullptr) { return false; }

		/** Protects device swap related resources during device swaps */
		virtual void LockDeviceSwap() {}

		/** Unprotects device swap resources. Balance with LockDeviceSwap() */
		virtual void UnlockDeviceSwap() {}

		/** Returns the platform device info of the currently open audio stream. */
		virtual FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const = 0;

		/** Submit the given buffer to the platform's output audio device. */
		virtual void SubmitBuffer(const uint8* Buffer) {};

		/** Submit a buffer that is to be output directly through a discreet device channel. */
		virtual void SubmitDirectOutBuffer(const int32 InDirectOutIndex, const Audio::FAlignedFloatBuffer& InBuffer) {};

		/** Allows platforms to filter the requested number of frames to render. Some platforms only support specific frame counts. */
		virtual int32 GetNumFrames(const int32 InNumReqestedFrames) { return InNumReqestedFrames; }

		/** Whether or not the platform disables caching of decompressed PCM data (i.e. to save memory on fixed memory platforms) */
		UE_DEPRECATED(5.7, "PCM Audio Caching is a leagacy feature that is now disabled for all platforms")
		virtual bool DisablePCMAudioCaching() const { return true; }

		/** Whether or not this platform has hardware decompression. */
		virtual bool SupportsHardwareDecompression() const { return false; }

		/** Whether this is an interface for a non-realtime renderer. If true, synch events will behave differently to avoid deadlocks. */
		virtual bool IsNonRealtime() const { return false; }

		/** Return any optional device name defined in platform configuratio. */
		virtual FString GetDefaultDeviceName() = 0;

		// Helper function to gets the channel map type at the given index.
		AUDIOMIXERCORE_API static bool GetChannelTypeAtIndex(const int32 Index, EAudioMixerChannel::Type& OutType);

        // Function to stop all audio from rendering. Used on mobile platforms which can suspend the application.
        virtual void SuspendContext() {}
        
        // Function to resume audio rendering. Used on mobile platforms which can suspend the application.
        virtual void ResumeContext() {}
        
		// Function called at the beginning of every call of UpdateHardware on the audio thread.
		virtual void OnHardwareUpdate() {}

		// Get the DeviceInfo Cache if one exists.
		virtual IAudioPlatformDeviceInfoCache* GetDeviceInfoCache() const { return nullptr;  }

		// Determine if the given device info struct is valid for use with this platform
		virtual bool IsDeviceInfoValid(const FAudioPlatformDeviceInfo& InDeviceInfo) const { return false; }

		// Subclasses can override to determine if it should use Cache for DeviceInfo Enumeration
		virtual bool ShouldUseDeviceInfoCache() const { return false; }
	
	public: // Public Functions
		//~ Begin FRunnable
		AUDIOMIXERCORE_API uint32 Run() override;
		//~ End FRunnable

		/**
		*  FSingleThreadRunnable accessor for ticking this FRunnable when multi-threading is disabled.
		*  @return FSingleThreadRunnable Interface for this FRunnable object.
		*/
		virtual class FSingleThreadRunnable* GetSingleThreadInterface() override { return this; }

		//~ Begin FSingleThreadRunnable Interface
		AUDIOMIXERCORE_API virtual void Tick() override;
		//~ End FSingleThreadRunnable Interface

		/** Constructor. */
		AUDIOMIXERCORE_API IAudioMixerPlatformInterface();

		/** Retrieves the next generated buffer and feeds it to the platform mixer output stream. */
		AUDIOMIXERCORE_API void ReadNextBuffer();

		/** Reset the fade state (use if reusing audio platform interface, e.g. in main audio device. */
		AUDIOMIXERCORE_API virtual void FadeIn();

		/** Start a fadeout. Prevents pops during shutdown. */
		AUDIOMIXERCORE_API virtual void FadeOut();

		/** Returns the last error generated. */
		FString GetLastError() const { return LastError; }

		/** This is called after InitializeHardware() is called. */
		AUDIOMIXERCORE_API void PostInitializeHardware();

		/** Used to determine if this object is listening to system device change events. */
		bool GetIsListeningForDeviceEvents() const { return bIsListeningForDeviceEvents; }
		void SetIsListeningForDeviceEvents(bool bInListeningForDeviceEvents) { bIsListeningForDeviceEvents = bInListeningForDeviceEvents; }

	protected:
		
		// Run the "main" audio device
		AUDIOMIXERCORE_API uint32 MainAudioDeviceRun();
		
		// Wrapper around the thread Run. This is virtualized so a platform can fundamentally override the render function.
		AUDIOMIXERCORE_API virtual uint32 RunInternal();

		/** Is called when an error, warning or log is generated. */
		inline void AudioMixerPlatformLogOnce(const FString& LogDetails, const FString& FileName, int32 LineNumber, ELogVerbosity::Type InVerbosity = ELogVerbosity::Error)
		{
#if !NO_LOGGING
			// Log once to avoid Spam.
			static FCriticalSection Cs;
			static TSet<uint32> LogHistory;

			FScopeLock Lock(&Cs);
			FString Message = FString::Printf(TEXT("Audio Platform Device: %s (File %s, Line %d)"), *LogDetails, *FileName, LineNumber);

			if ((ELogVerbosity::Error == InVerbosity) || (ELogVerbosity::Fatal == InVerbosity))
			{
				// Save last error if it was at the error level.
				LastError = Message;
			}

			uint32 Hash = GetTypeHash(Message);
			if (!LogHistory.Contains(Hash))
			{
				switch (InVerbosity)
				{
					case ELogVerbosity::Fatal:
						UE_LOGF(LogAudioMixer, Fatal, "%ls", *Message);
						break;

					case ELogVerbosity::Error:
						UE_LOGF(LogAudioMixer, Error, "%ls", *Message);
						break;

					case ELogVerbosity::Warning:
						UE_LOGF(LogAudioMixer, Warning, "%ls", *Message);
						break;

					case ELogVerbosity::Display:
						UE_LOGF(LogAudioMixer, Display, "%ls", *Message);
						break;

					case ELogVerbosity::Log:
						UE_LOGF(LogAudioMixer, Log, "%ls", *Message);
						break;

					case ELogVerbosity::Verbose:
						UE_LOGF(LogAudioMixer, Verbose, "%ls", *Message);
						break;

					case ELogVerbosity::VeryVerbose:
						UE_LOGF(LogAudioMixer, VeryVerbose, "%ls", *Message);
						break;

					default:
						UE_LOGF(LogAudioMixer, Error, "%ls", *Message);
						{
							static_assert(static_cast<uint8>(ELogVerbosity::NumVerbosity) == 8, "Missing ELogVerbosity case coverage");
						}
						break;
				}
				
				LogHistory.Add(Hash);
			}
#endif
		}



		/** Start generating audio from our mixer. */
		AUDIOMIXERCORE_API void BeginGeneratingAudio();

		/** Stops the render thread from generating audio. */
		AUDIOMIXERCORE_API void StopGeneratingAudio();

		/** Non-blocking: signals the render thread to stop (sets Stopping state, triggers event)
		 *  without waiting for the thread to exit. Use TryFinishStopGeneratingAudio() to complete cleanup. */
		AUDIOMIXERCORE_API void SignalStopGeneratingAudio();

		/** Checks if the render thread has stopped after SignalStopGeneratingAudio() was called.
		 *  If the thread has stopped (or a timeout is reached), cleans up thread/events and returns true.
		 *  If the thread is still running, returns false. */
		AUDIOMIXERCORE_API bool TryFinishStopGeneratingAudio();

		// Deprecated - use ApplyPrimaryAttenuation
		UE_DEPRECATED(5.1, "ApplyMasterAttenuation is deprecated, please use ApplyPrimaryAttenuation instead.")
		AUDIOMIXERCORE_API void ApplyMasterAttenuation(TArrayView<const uint8>& InOutPoppedAudio);

		/** Performs buffer fades for shutdown/startup of audio mixer. */
		AUDIOMIXERCORE_API void ApplyPrimaryAttenuation(TArrayView<const uint8>& InOutPoppedAudio);

		template<typename BufferType>
		void ApplyAttenuationInternal(TArrayView<BufferType>& InOutBuffer);

		/** When called, spins up a thread to start consuming output when no audio device is available. */
		AUDIOMIXERCORE_API void StartRunningNullDevice();

		/** When called, terminates the null device. */
		AUDIOMIXERCORE_API void StopRunningNullDevice();
		
		/** Called by platform specific logic to pre-create or create the null renderer thread  */
		AUDIOMIXERCORE_API void CreateNullDeviceThread(const TFunction<void()> InCallback, float InBufferDuration, bool bShouldPauseOnStart);

	protected:

		/** The audio device stream info. */
		FAudioOutputStreamInfo AudioStreamInfo;
		FAudioMixerOpenStreamParams OpenStreamParams;

		/** List of generated output buffers. */
		Audio::FOutputBuffer OutputBuffer;

		/** Whether or not we warned of buffer underrun. */
		bool bWarnedBufferUnderrun;
		int32 UnderrunCount = 0;
		int32 CurrentUnderrunCount = 0;
		uint64 TimeLastWarningCycles = 0;

		/** The audio render thread. */
		//FRunnableThread* AudioRenderThread;
		TUniquePtr<FRunnableThread> AudioRenderThread;

		/** The render thread sync event. */
		FEvent* AudioRenderEvent;

		/** Critical Section used for times when we need the render loop to halt for the device swap. */
		FCriticalSection DeviceSwapCriticalSection;

		/** This is used if we are attempting to TryLock on DeviceSwapCriticalSection, but a buffer callback is being called in the current thread. */
		UE_DEPRECATED(5.6, "bIsInDeviceSwap has been deprecated. AudioStreamInfo.StreamState can provide similar functionality.")
		FThreadSafeBool bIsInDeviceSwap;

		/** Event allows you to block until fadeout is complete. */
		FEvent* AudioFadeEvent;

		/** The number of mixer buffers to queue on the output source voice. */
		int32 NumOutputBuffers;

		/** The fade value. Used for fading in/out primary audio. */
		float FadeVolume;

		/** Source param used to fade in and out audio device. */
		FParam FadeParam;

		/** This device name can be used to override the default device being used on platforms that use strings to identify audio devices. */
		FString CurrentDeviceName;

		/** String containing the last generated error. */
		FString LastError;

		int32 CallCounterApplyAttenuationInternal{ 0 };
		int32 CallCounterReadNextBuffer{ 0 };

		FThreadSafeBool bPerformingFade;
		FThreadSafeBool bFadedOut;
		FThreadSafeBool bIsDeviceInitialized;

		UE_DEPRECATED(5.6, "bMoveAudioStreamToNewAudioDevice has been deprecated. AudioStreamInfo.StreamState can provide similar functionality.")
		FThreadSafeBool bMoveAudioStreamToNewAudioDevice;

		FThreadSafeBool bIsUsingNullDevice;
		FThreadSafeBool bIsGeneratingAudio;

		/** Timestamp (seconds) when SignalStopGeneratingAudio was called, for timeout detection. */
		double StopGeneratingAudioSignalTime = 0.0;

		/** A Counter to provide the next unique id. */
		AUDIOMIXERCORE_API static FThreadSafeCounter NextInstanceID;

		/** A Unique ID Identifying this instance. Mostly used for logging. */ 
		const int32 InstanceID{ -1 };

	private:
		TUniquePtr<FMixerNullCallback> NullDeviceCallback;
		
		/** Used to determine if this object is listening to system device change events. */
		bool bIsListeningForDeviceEvents = true;

	};

	/** Audio mixer platform objects can subclass this in order to add device swap capabilities. */
	class FAudioMixerPlatformSwappable : public IAudioMixerPlatformInterface
	{
	public:
		UE_API FAudioMixerPlatformSwappable();
		virtual ~FAudioMixerPlatformSwappable() override = default;

		//~ Begin IAudioMixerPlatformInterface
		UE_API virtual bool RequestDeviceSwap(const FString& DeviceID, const bool bInForce, const TCHAR* InReason) override;
		virtual void LockDeviceSwap() override { DeviceSwapCriticalSection.Lock(); }
		virtual void UnlockDeviceSwap() override { DeviceSwapCriticalSection.Unlock(); }
		UE_API virtual bool CheckAudioDeviceChange() override;
		UE_API virtual bool MoveAudioStreamToNewAudioDevice() override;
		UE_API virtual void ResumePlaybackOnNewDevice() override;
		//~ End IAudioMixerPlatformInterface

		/** Called to determine if the current device swap request should be allowed to proceed. */
		UE_API virtual bool AllowDeviceSwap(const bool bInForceSwap);

		/** Initializes a new device swap context with the given parameters */
		virtual bool InitializeDeviceSwapContext(const FString& InRequestedDeviceID, const TCHAR* InReason) = 0;

		/** Called repeatedly to update an active, async device swap */
		UE_API virtual bool CheckThreadedDeviceSwap();

		/** Called at the beginning of a device swap to perform any needed initialization */
		virtual bool PreDeviceSwap() { return true; }

		/** Kicks of an async device swap task */
		virtual void EnqueueAsyncDeviceSwap() = 0;

		/** Performs a device swap synchronously in the current thread */
		virtual void SynchronousDeviceSwap() = 0;

		/** Called after a device swap completes, providing an opportunity for any needed cleanup */
		virtual bool PostDeviceSwap() { return true; }

	protected:
		/** Used in OnDeviceAdded for determining if an added device is the same as the original device. */
		FString GetOriginalAudioDeviceId() const { return OriginalAudioDeviceId; }
		void SetOriginalAudioDeviceId(const FString& InAudioDeviceId) { OriginalAudioDeviceId = InAudioDeviceId; }
		
		/** Future which holds result of device swap upon completion. */
		void SetActiveDeviceSwapFuture(TFuture<TUniquePtr<FDeviceSwapResult>>&& InFuture) { ActiveDeviceSwap = MoveTemp(InFuture); } 
		void ResetActiveDeviceSwapFuture() { ActiveDeviceSwap.Reset(); }

		/** The results produced from a device swap operation. The containing future retains memory ownership.
		 *  Will be valid until PostDeviceSwap returns.
		 */
		const FDeviceSwapResult* GetDeviceSwapResult() const { return ActiveDeviceSwap.IsValid() ? ActiveDeviceSwap.Get().Get() : nullptr; }
		FDeviceSwapResult* GetDeviceSwapResult() { return ActiveDeviceSwap.IsValid() ? ActiveDeviceSwap.Get().Get() : nullptr; }

	private:
		/** Used in OnDeviceAdded for determining if an added device is the same as the original device. */
		FString OriginalAudioDeviceId;

		/** Future which holds result of device swap upon completion. */
		TFuture<TUniquePtr<FDeviceSwapResult>> ActiveDeviceSwap;
		
		/** Used to rate limit device swap attempts to ignore multiple requests */
		double LastDeviceSwapTime = 0.0;

		/** True when we've signaled the render thread to stop but haven't finished cleanup yet.
		 *  Only read/written on the audio thread (inside CheckThreadedDeviceSwap). */
		bool bWaitingForRenderThreadStop = false;

		/** If a force device swap is requested while bWaitingForRenderThreadStop is true, the
		 *  request is deferred here so it can be replayed after the current swap completes.
		 *  Empty string means no deferred swap; non-empty or explicit default ("") is stored
		 *  as a TOptional to distinguish "no request" from "request for default device". */
		TOptional<FString> DeferredDeviceSwapId;
		FString DeferredDeviceSwapReason;
	};

	class FScopedDeviceSwapLock final
	{
	public:
		FScopedDeviceSwapLock() = delete;
		explicit FScopedDeviceSwapLock(IAudioMixerPlatformInterface* InPlatform)
			: AudioMixerPlatform(InPlatform)
		{
			if (AudioMixerPlatform)
			{
				AudioMixerPlatform->LockDeviceSwap();
			}
		}

		~FScopedDeviceSwapLock()
		{
			if (AudioMixerPlatform)
			{
				AudioMixerPlatform->UnlockDeviceSwap();
			}
		}

	private:
		IAudioMixerPlatformInterface* AudioMixerPlatform = nullptr;
	};
}

/**
 * Interface for audio device modules
 */

class FAudioDevice;

/** Defines the interface of a module implementing an audio device and associated classes. */
class IAudioDeviceModule : public IModuleInterface
{
public:

	/** Creates a new instance of the audio device implemented by the module. */
	virtual bool IsAudioMixerModule() const { return false; }
	/** Does this class of device support multiclient access to the driver */
	virtual bool IsAudioDeviceClassMulticlient() const { return true; }
	virtual FAudioDevice* CreateAudioDevice() { return nullptr; }
	virtual Audio::IAudioMixerPlatformInterface* CreateAudioMixerPlatformInterface() { return nullptr; }
};

#undef UE_API
