// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"

namespace EAudioMixerChannel
{
	/** Enumeration values represent sound file or speaker channel types. */
	enum Type
	{
		FrontLeft,
		FrontRight,
		FrontCenter,
		LowFrequency,
		BackLeft,
		BackRight,
		FrontLeftOfCenter,
		FrontRightOfCenter,
		BackCenter,
		SideLeft,
		SideRight,
		TopCenter,
		TopFrontLeft,
		TopFrontCenter,
		TopFrontRight,
		TopBackLeft,
		TopBackCenter,
		TopBackRight,
		Unknown,
		ChannelTypeCount,
		DefaultChannel = FrontLeft
	};

// Foreach for each speaker (minus ChannelTypeCount and Default).
// Not reflected enum, so do this by hand.
#define FOREACH_EAUDIOMIXERCHANNEL(OP)\
	OP(FrontLeft)\
	OP(FrontRight)\
	OP(FrontCenter)\
	OP(LowFrequency)\
	OP(BackLeft)\
	OP(BackRight)\
	OP(FrontLeftOfCenter)\
	OP(FrontRightOfCenter)\
	OP(BackCenter)\
	OP(SideLeft)\
	OP(SideRight)\
	OP(TopCenter)\
	OP(TopFrontLeft)\
	OP(TopFrontCenter)\
	OP(TopFrontRight)\
	OP(TopBackLeft)\
	OP(TopBackCenter)\
	OP(TopBackRight)\
	OP(Unknown)
	
	inline constexpr int32 MaxSupportedChannel = TopCenter;

	constexpr const TCHAR* ToString(const Type InType) 
	{
		// Define using FOREACH macro 
		#define CASE_TO_STRING(X) case X: return TEXT(#X);
		switch (InType)
		{
			FOREACH_EAUDIOMIXERCHANNEL(CASE_TO_STRING)
			default: break;
		}
		#undef CASE_TO_STRING
		return TEXT("UNSUPPORTED");
	}
	
	inline FName ToName(const Type InType)
	{
		// Keep these static.
		#define CASE_TO_NAME(X) case X:\
		{\
			static FName Name(ToString(X));\
			return Name;\
		}
		
		// Define using FOREACH macro
		switch (InType)
		{
			FOREACH_EAUDIOMIXERCHANNEL(CASE_TO_NAME)
			default: break;
		}
		return TEXT("Unknown");
	
		#undef CASE_TO_NAME
	}
	
	inline TOptional<Type> FromName(const FName InType)
	{
		// Define using FOREACH macro.
		#define COMPARE_RETURN_NAME(X) if (InType == ToName(X)) return X;
		FOREACH_EAUDIOMIXERCHANNEL(COMPARE_RETURN_NAME)
		#undef COMPARE_RETURN_NAME
		return {}; // Fail.
	}
}
