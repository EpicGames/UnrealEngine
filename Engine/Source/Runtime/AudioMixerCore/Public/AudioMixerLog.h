// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

AUDIOMIXERCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioMixer, Log, All);
AUDIOMIXERCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioMixerDebug, Warning, All);
