// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "HAL/Platform.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

#include <atomic>

class FEvent;

namespace Audio
{
	/**
	 * FMixerNullCallback
	 * This class, when started, spawns a new high priority thread that exists to query an FAudioMixerPlatformInterface
	 * and immediately throw out whatever buffers it receives.
	 */
	class FMixerNullCallback : protected FRunnable
	{
	public:

		/**
		 * Constructing the FMixerNullCallback immediately begins calling
		 * InCallback every BufferDuration seconds.
		 */
		AUDIOMIXERCORE_API FMixerNullCallback(float BufferDuration, TFunction<void()> InCallback, EThreadPriority ThreadPriority = TPri_TimeCritical, bool bStartedPaused = false);

		/**
		 * The destructor waits on Callback to be completed before stopping the thread.
		 */
		virtual ~FMixerNullCallback() = default;

		// FRunnable override:
		AUDIOMIXERCORE_API virtual uint32 Run() override;
		AUDIOMIXERCORE_API virtual void Stop() override;

		// Resume a paused null renderer. 
		AUDIOMIXERCORE_API void Resume(const TFunction<void()>& InCallback, float InBufferDuration);

		// Pause the thread, making it sleep until woken, not consuming cycles or buffers.
		AUDIOMIXERCORE_API void Pause();

	private:

		// Default constructor intentionally suppressed:
		FMixerNullCallback() = delete;

		// Callback used.
		TFunction<void()> Callback;

		// Used to determine amount of time we should wait between callbacks.
		float CallbackTime;

		// Flagged on Stop
		std::atomic<bool> bShouldShutdown;
		std::atomic<bool> bShouldRecyle;
		FEvent* SleepEvent = nullptr;
		FEvent* WakeupEvent = nullptr;	
		TUniquePtr<FRunnableThread> CallbackThread;
		double LastLog = 0.f;
	};
}

