// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/Internationalization.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Trace/Trace.h"

#ifndef	UE_AUDIO_PROFILERTRACE_ENABLED
	#define UE_AUDIO_PROFILERTRACE_ENABLED UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

#if	UE_AUDIO_PROFILERTRACE_ENABLED
	#define AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(Name) TRACE_CPUPROFILER_EVENT_SCOPE(Name)
	AUDIOMIXERCORE_API UE_TRACE_CHANNEL_EXTERN(AudioChannel);
	AUDIOMIXERCORE_API UE_TRACE_CHANNEL_EXTERN(AudioMixerChannel);
#else
	#if CPUPROFILERTRACE_ENABLED
		#define AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(Name) TRACE_CPUPROFILER_EVENT_SCOPE(Name)
	#else // !CPUPROFILERTRACE_ENABLED
		#define AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(Name)
	#endif
#endif


// Audio Insights Event Log in-built event types
// actual event list is in AudioMixer.cpp
namespace Audio::Trace::EventLog::ID
{
	// Active Sound : Playing Audio
	extern AUDIOMIXERCORE_API const FString SoundStart;
	extern AUDIOMIXERCORE_API const FString SoundStop;

	// Virtualization
	extern AUDIOMIXERCORE_API const FString SoundVirtualized;
	extern AUDIOMIXERCORE_API const FString SoundRealized;

	// Triggers
	extern AUDIOMIXERCORE_API const FString PlayRequestSoundHandle;
	extern AUDIOMIXERCORE_API const FString StopRequestedSoundHandle;
	
	extern AUDIOMIXERCORE_API const FString PlayRequestAudioComponent;
	extern AUDIOMIXERCORE_API const FString StopRequestAudioComponent;
	
	extern AUDIOMIXERCORE_API const FString PlayRequestOneShot;
	extern AUDIOMIXERCORE_API const FString PlayRequestSoundAtLocation;
	extern AUDIOMIXERCORE_API const FString PlayRequestSound2D;
	extern AUDIOMIXERCORE_API const FString PlayRequestSlateSound;
	
	// Active Sound : Playing Audio Failures
	extern AUDIOMIXERCORE_API const FString PlayFailedNotPlayable;
	extern AUDIOMIXERCORE_API const FString PlayFailedOutOfRange;
	extern AUDIOMIXERCORE_API const FString PlayFailedDebugFiltered;
	extern AUDIOMIXERCORE_API const FString PlayFailedConcurrency;

	extern AUDIOMIXERCORE_API const FString StopRequestActiveSound;
	extern AUDIOMIXERCORE_API const FString StopRequestSoundsUsingResource;
	extern AUDIOMIXERCORE_API const FString StopRequestConcurrency;
	
	extern AUDIOMIXERCORE_API const FString PauseSoundRequested;
	extern AUDIOMIXERCORE_API const FString ResumeSoundRequested;
	
	extern AUDIOMIXERCORE_API const FString StopAllRequested;
	extern AUDIOMIXERCORE_API const FString FlushAudioDeviceRequested;

	// PIE Session
	extern AUDIOMIXERCORE_API const FString PIEStarted;
	extern AUDIOMIXERCORE_API const FString PIEStopped;
}
