// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

namespace Audio {

	namespace EAudioMixerStreamDataFormat
	{
		enum Type
		{
			Unknown,
			Float,
			Int16,
			Unsupported
		};
	}

	/**
	 * EAudioOutputStreamState
	 * Specifies the state of the output audio stream.
	 */
	namespace EAudioOutputStreamState
	{
		enum Type
		{
			/* The audio stream is shutdown or not uninitialized. */
			Closed,
		
			/* The audio stream is open but not running. */
			Open,

			/** The audio stream is open but stopped. */
			Stopped,
		
			/** The audio output stream is stopping. */
			Stopping,

			/** The audio output stream is open and running. */
			Running,

			/** The audio device hardware is being swapped */
			SwappingDevice,
		};
	}

	// Indicates a platform-specific format
	inline FName NAME_PLATFORM_SPECIFIC("PLATFORM_SPECIFIC");
	inline FName NAME_PROJECT_DEFINED("PROJECT_DEFINED");

	// Supported on all platforms:
	inline FName NAME_BINKA("BINKA");
	inline FName NAME_ADPCM("ADPCM");
	inline FName NAME_PCM("PCM");
	inline FName NAME_OPUS("OPUS");
	inline FName NAME_RADA("RADA");

	// Not yet supported on all platforms as a selectable option so is included under "platform specific" enumeration for now. 
	inline FName NAME_OGG("OGG");
}

struct FAudioPlatformSettings
{
	/** Sample rate to use on the platform for the mixing engine. Higher sample rates will incur more CPU cost. */
	int32 SampleRate;

	/** The amount of audio to compute each callback block. Lower values decrease latency but may increase CPU cost. */
	int32 CallbackBufferFrameSize;

	/** The number of buffers to keep enqueued. More buffers increases latency, but can compensate for variable compute availability in audio callbacks on some platforms. */
	int32 NumBuffers;

	/** The max number of channels (simultaneous voices) to use as the limit for this platform. If given a value of 0, it will use the value from the active Global Audio Quality Settings */
	int32 MaxChannels;

	/** The number of workers to use to compute source audio. Will only use up to the max number of sources. Will evenly divide sources to each source worker. */
	int32 NumSourceWorkers;

	static AUDIOMIXERCORE_API FAudioPlatformSettings GetPlatformSettings(const TCHAR* PlatformSettingsConfigFile);

	FAudioPlatformSettings()
		: SampleRate(48000)
		, CallbackBufferFrameSize(1024)
		, NumBuffers(2)
		, MaxChannels(0) // This needs to be 0 to indicate it's not overridden from the audio settings object, which is the default used on all platforms
		, NumSourceWorkers(0)
	{
	}
};
